// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Data;
using System.Drawing;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Windows.Forms;
using System.IO;                // Directory.
using System.Diagnostics;       // Debug.
using System.Drawing.Imaging;   // ImageData
using System.Runtime.InteropServices;   // DllImport
using System.Xml.Serialization;
using System.Net.Mail;


namespace ImageValidator
{
    public partial class Form1 : Form, IMessageFilter 
    {
        //
        ImageValidatorData data;
        //
        ImageValidatorIntermediate intermediate;

        // The column we are currently using for sorting.
        private ColumnHeader SortingColumn = null;

        // -1 if there is no drag
        private int MouseStartTrackOffsetX;
        private int MouseStartTrackOffsetY;

        private float ImageOffsetX;
        private float ImageOffsetY;
        // 0 means 1:1, 120 means down sample
        private int ImageZoom;

        // --------------------------------------------------------------------

        // to redirect Mouse wheel message to the control under it, not the control in focus
        // from http://stackoverflow.com/questions/4769854/windows-forms-capturing-mousewheel

        public bool PreFilterMessage(ref Message m)
        {
            if (m.Msg == 0x20a)
            {
                // WM_MOUSEWHEEL, find the control at screen position m.LParam
                Point pos = new Point(m.LParam.ToInt32() & 0xffff, m.LParam.ToInt32() >> 16);
                IntPtr hWnd = WindowFromPoint(pos);
                if (hWnd != IntPtr.Zero && hWnd != m.HWnd && Control.FromHandle(hWnd) != null)
                {
                    SendMessage(hWnd, m.Msg, m.WParam, m.LParam);
                    return true;
                }
            }
            return false;
        }
        // P/Invoke declarations
        [DllImport("user32.dll")]
        private static extern IntPtr WindowFromPoint(Point pt);
        [DllImport("user32.dll")]
        private static extern IntPtr SendMessage(IntPtr hWnd, int msg, IntPtr wp, IntPtr lp);

        // --------------------------------------------------------------------

        public Form1()
        {
            MouseStartTrackOffsetX = -1;

            InitializeComponent();
            Application.AddMessageFilter(this);
        }

        public void PopulateList()
        {
            intermediate = new ImageValidatorIntermediate();

            data.PopulateList(textBoxTestDir.Text, textBoxRefDir.Text);
            
            // sort by column 1 
            SetListViewSorting(1);

            {
                EventArgs e = null;
                Form1_Resize(this, e);
                listView1_Click(this, e);
            }
        }

        private void Form1_Load(object sender, EventArgs e)
        {
//            textBoxRefDir.Text = @"F:\Martin.Mittring_Z3941_Dev_Rendering\Samples\Sandbox\RenderTest\Saved\Automation\RenderOutputValidation\CL12";
//           textBoxTestDir.Text = @"F:\Martin.Mittring_Z3941_Dev_Rendering\Samples\Sandbox\RenderTest\Saved\Automation\RenderOutputValidation\CL13";

            SetupFileSystemWatcher();

            folderBrowserDialog1.RootFolder = Environment.SpecialFolder.MyComputer;
            folderBrowserDialog1.ShowNewFolderButton = false;
            //            folderBrowserDialog1.SelectedPath = @"F:/Martin.Mittring_Z3941_Dev_Rendering/Samples/Sandbox/RenderTest/Saved/Automation/RenderOutputValidation/CL224";

            listView1.AllowColumnReorder = true;
            listView1.FullRowSelect = true;
            listView1.GridLines = true;
            listView1.Columns.Add("Result (pixels failed)", 120, HorizontalAlignment.Left);
            listView1.Columns.Add("Matinee Location (in sec)", 140, HorizontalAlignment.Left);
//            listView1.Columns.Add("Actor", 100, HorizontalAlignment.Left);
//           listView1.Columns.Add("Map", 100, HorizontalAlignment.Left);
//            listView1.Columns.Add("Platform", 100, HorizontalAlignment.Left);
            listView1.Columns.Add("Folder Name", 120, HorizontalAlignment.Left);
            listView1.Columns.Add("File Name", 350, HorizontalAlignment.Left);

            // sort by second colunm
            {
                SortingColumn = listView1.Columns[1];
                listView1.Columns[1].Text = "> " + listView1.Columns[1].Text;
            }

            saveFileDialog1.FileName = "Default.IVxml";
            saveFileDialogExport.FileName = "ImageValidatorReport.html";

            PopulateList();

            // every 50ms we look for more work
            timer1.Interval = 50;

            timer1.Start();

            // load example, if not there, clear like "New"
            LoadSettings("Default.IVxml");
        }

        private int FindListViewIndexFromImageIndex(int ImageIndex)
        {
            int Ret = 0;
            foreach (ListViewItem item in this.listView1.Items)
            {
                if (item.ImageIndex == ImageIndex)
                {
                    break;
                }
                ++Ret;
            }
            return Ret;
        }

        private void buttonPickRefFolder_Click(object sender, EventArgs e)
        {
            folderBrowserDialog1.SelectedPath = textBoxRefDir.Text;
            if (folderBrowserDialog1.ShowDialog() == DialogResult.OK)
            {
                textBoxRefDir.Text = folderBrowserDialog1.SelectedPath;
                SetupFileSystemWatcher();
                PopulateList();
                // todo: Update ListView?
            }
        }

        private void buttonPickTestFolder_Click(object sender, EventArgs e)
        {
            folderBrowserDialog1.SelectedPath = textBoxTestDir.Text;
            if (folderBrowserDialog1.ShowDialog() == DialogResult.OK)
            {
                textBoxTestDir.Text = folderBrowserDialog1.SelectedPath;
                SetupFileSystemWatcher();
                PopulateList();
                // todo: Update ListView?
            }
        }

        private void textBoxTestDir_KeyDown(object sender, KeyEventArgs e)
        {
            if (e.KeyValue == 13)
            {
                PopulateList();
                SetupFileSystemWatcher();
                // todo: Update ListView?
            }
        }

        private void textBoxRefDir_KeyDown(object sender, KeyEventArgs e)
        {
            if (e.KeyValue == 13)
            {
                PopulateList();
                SetupFileSystemWatcher();
                // todo: Update ListView?
            }
        }

        private void UpdateListView()
        {
            // select first element if no element is currently selected
            int SelectedImageIndex = 0;

            if (listView1.SelectedIndices.Count == 1)
            {
                SelectedImageIndex = listView1.Items[listView1.SelectedIndices[0]].ImageIndex;
                listView1.SelectedIndices.Clear();
            }

            listView1.Items.Clear();
            
            ImageValidatorSettings settings = GetSettings();

            int Index = 0;
            foreach (ImageValidatorData.ImageEntry entry in data.imageEntries)
            {
                ImageValidatorData.ImageEntryColumnData columnData = new ImageValidatorData.ImageEntryColumnData(entry.Name);

                string ResultString = "?";

                if (entry.testResult != null)
                {
                    ResultString = entry.testResult.GetString(ref settings);
                }
                ListViewItem item = new ListViewItem(ResultString, Index++);
                item.SubItems.Add(columnData.Time);
//                item.SubItems.Add((1.0f + 0.1f * Index).ToString());
//                item.SubItems.Add(columnData.Actor);
//                item.SubItems.Add(columnData.Map);
//                item.SubItems.Add(columnData.Platform);
                item.SubItems.Add(Path.GetDirectoryName(entry.Name));
                item.SubItems.Add(Path.GetFileName(entry.Name));
                if (entry.testResult != null)
                {
                    item.BackColor = entry.testResult.GetColor(ref settings);
                }
              
                listView1.Items.Add(item);
            }

            if (listView1.Items.Count > 0)
            {
                listView1.SelectedIndices.Add(FindListViewIndexFromImageIndex(SelectedImageIndex));
            }
        }

        // @return 0 if nothing is selected
        private int GetSelectedListViewIndex()
        {
            ListView.SelectedIndexCollection indexes = listView1.SelectedIndices;

            return (indexes.Count == 1) ? indexes[0] : 0;
        }

        // @return 0 if nothing is there
        private int GetSelectedImageEntryIndex()
        {
            int ListViewIndex = GetSelectedListViewIndex();

            return listView1.Items.Count != 0 ? listView1.Items[ListViewIndex].ImageIndex : 0;
        }

        private ImageValidatorSettings GetSettings()
        {
            ImageValidatorSettings settings = new ImageValidatorSettings();

            settings.TestDir = textBoxTestDir.Text;
            settings.RefDir = textBoxRefDir.Text;

            {
                int Value;
                Int32.TryParse(this.textBoxThreshold.Text, out Value);

                if (Value > 0)
                {
                    settings.Threshold = (uint)Value;
                }
            }

            {
                int Value;
                Int32.TryParse(this.textBoxCountToFail.Text, out Value);

                if (Value > 0)
                {
                    settings.PixelCountToFail = (uint)Value;
                }
            }


            /*
            if (settings.TestDir == settings.RefDir)
            {
                // it doesn't make sense to compare the same folders
                settings.TestDir = "";
                settings.RefDir = "";
            }
            */

            return settings;
        }

        private void SetSettings(ImageValidatorSettings settings)
        {
            textBoxTestDir.Text = settings.TestDir;
            textBoxRefDir.Text = settings.RefDir;
            textBoxThreshold.Text = settings.Threshold.ToString();
            textBoxCountToFail.Text = settings.PixelCountToFail.ToString();
        }

        private void UpdateImages(int ImageEntryIndex)
        {
            if (data.imageEntries.Count != 0)
            {
                intermediate.Process(GetSettings(), data.imageEntries[ImageEntryIndex]);
            }
            pictureBoxTest.Invalidate();
            pictureBoxDiff.Invalidate();
            pictureBoxRef.Invalidate();
            UpdateListView();
        }

        private void listView1_Click(object sender, EventArgs e)
        {
            UpdateImages(GetSelectedImageEntryIndex());
            Form1_Resize(sender, e);
        }

        private float ComputeScale()
        {
            System.Drawing.Size imageSize = intermediate.GetImagesSize();

            if (imageSize.Width > 0 && imageSize.Height > 0)
            {
                return (float)Math.Pow(2, ImageZoom / 120) * pictureBoxTest.Width / (float)imageSize.Width;
            }
            else
            {
                return 1.0f;
            }
        }

        private void DrawOutlinedString(Graphics g, int x, int y, string drawString, Font drawFont)
        {
            SolidBrush outlineBrush = new SolidBrush(Color.Black);
            SolidBrush drawBrush = new SolidBrush(Color.White);

            const int r = 2;
            // bold black border around the text to make it more readable
            for (int dy = -r; dy <= r; ++dy)
            for (int dx = -r; dx <= r; ++dx)
            {
                if(dx != 0 && dy != 0)
                {
                    g.DrawString(drawString, drawFont, outlineBrush, new PointF(x + dx, y + dy));
                }
            }

            g.DrawString(drawString, drawFont, drawBrush, new PointF(x + 0, y + 0));
        }

        private void PaintImage(Bitmap image, object sender, PaintEventArgs e, string drawString)
        {
            Graphics g = e.Graphics;
            PictureBox pictureBox = sender as PictureBox;

            if (image != null)
            {
                GraphicsUnit Unit = GraphicsUnit.Pixel;
                RectangleF bounds = image.GetBounds(ref Unit);
                
                float Scale = ComputeScale();

                g.InterpolationMode = System.Drawing.Drawing2D.InterpolationMode.NearestNeighbor;

                RectangleF destRect = new RectangleF();

                destRect.Size = new SizeF(bounds.Width * Scale, bounds.Height * Scale);
                destRect.Location = new PointF(-ImageOffsetX * Scale, -ImageOffsetY * Scale);

                g.DrawImage(image, destRect, bounds, Unit);
            }

            Font drawFont = new Font("Arial", 12);

            SizeF textSize = g.MeasureString(drawString, drawFont);
            DrawOutlinedString(g, (int)(pictureBox.Width / 2 - textSize.Width / 2), (int)(pictureBox.Height - textSize.Height), drawString, drawFont);
        }

        private void pictureBoxTest_Paint(object sender, PaintEventArgs e)
        {
            PaintImage(intermediate.imageTest, sender, e, "Test");
        }

        private void pictureBoxDiff_Paint(object sender, PaintEventArgs e)
        {
            PaintImage(intermediate.imageDiff, sender, e, "Difference");
        }

        private void pictureBoxRef_Paint(object sender, PaintEventArgs e)
        {
            PaintImage(intermediate.imageRef, sender, e, "Reference");
        }

        private void Form1_Resize(object sender, EventArgs e)
        {
            // better with try
            // http://stackoverflow.com/questions/16441643/how-to-avoid-screen-flickering

//            this.SuspendLayout(); 

            int totalWidth = ClientSize.Width;
            int totalHeight = ClientSize.Height;

            int border = pictureBoxTest.Location.X;

            // 4 borders: left, right, between test and diff, between diff and ref
            int imagewidth = (totalWidth - 4 * border) / 3;

            if (imagewidth < 0) imagewidth = 0;

            System.Drawing.Size imageSize = intermediate.GetImagesSize();

            // Y / X
            float AspectRatio = imageSize.Height / (float)imageSize.Width;

            int imageheight = (int)(imagewidth * AspectRatio);

            int x0 = border;
            int x1 = x0 + imagewidth + border;
            int x2 = totalWidth - imagewidth - border;

            int y = totalHeight - imageheight - border;

            pictureBoxTest.Location = new System.Drawing.Point(x0, y);
            pictureBoxTest.Size = new System.Drawing.Size(imagewidth, imageheight);
            pictureBoxDiff.Location = new System.Drawing.Point(x1, y);
            pictureBoxDiff.Size = new System.Drawing.Size(imagewidth, imageheight);
            pictureBoxRef.Location = new System.Drawing.Point(x2, y);
            pictureBoxRef.Size = new System.Drawing.Size(imagewidth, imageheight);

            int listHeight = y - listView1.Location.Y - border;
            
            if (listHeight < 0) listHeight = 0;

            listView1.Size = new System.Drawing.Size(totalWidth - 2 * border, listHeight);

//            this.ResumeLayout(); 
        }

        private void textBoxCountToFail_TextChanged(object sender, EventArgs e)
        {
            UpdateListView();
        }

        private void textBoxThreshold_TextChanged(object sender, EventArgs e)
        {
            PopulateList();
            UpdateImages(GetSelectedImageEntryIndex());
        }

        private void ClearListViewSorting()
        {
            if (SortingColumn != null)
            {
                // Remove the old sort indicator.
                SortingColumn.Text = SortingColumn.Text.Substring(2);
                SortingColumn = null;
            }
        }

        private void SetListViewSorting(int ColumnIndex)
        {
            ClearListViewSorting();

            // sort by the specified column
            {
                ColumnClickEventArgs e = new ColumnClickEventArgs(ColumnIndex);

                listView1_ColumnClick(listView1, e);
            }
        }

        private void listView1_ColumnClick(object sender, ColumnClickEventArgs e)
        {
            // Sort on this column
            // from http://csharphelper.com/blog/2014/09/sort-a-listview-using-the-column-you-click-in-c

            // Get the new sorting column.
            ColumnHeader new_sorting_column = listView1.Columns[e.Column];

            // Figure out the new sorting order.
            System.Windows.Forms.SortOrder sort_order;
            if (SortingColumn == null)
            {
                // New column. Sort ascending.
                sort_order = SortOrder.Ascending;
            }
            else
            {
                // See if this is the same column.
                if (new_sorting_column == SortingColumn)
                {
                    // Same column. Switch the sort order.
                    if (SortingColumn.Text.StartsWith("> "))
                    {
                        sort_order = SortOrder.Descending;
                    }
                    else
                    {
                        sort_order = SortOrder.Ascending;
                    }
                }
                else
                {
                    // New column. Sort ascending.
                    sort_order = SortOrder.Ascending;
                }

                ClearListViewSorting();
            }

            // Display the new sort order.
            SortingColumn = new_sorting_column;
            if (sort_order == SortOrder.Ascending)
            {
                SortingColumn.Text = "> " + SortingColumn.Text;
            }
            else
            {
                SortingColumn.Text = "< " + SortingColumn.Text;
            }

            // Create a comparer
            listView1.ListViewItemSorter =
                new ListViewComparer(e.Column, sort_order);

            // Sort
            listView1.Sort();
        }

        private void timer1_Tick(object sender, EventArgs e)
        {
            if (data.ProcessOneElement(GetSettings()))
            {
                UpdateListView();
            }
        }

        private void SetupFileSystemWatcher()
        {
            // At the moment the feature is not stable so it's deactivated
            if(false)
            {
                ImageValidatorSettings settings = GetSettings();

                try
                {
                    fileSystemWatcher1.Path = settings.TestDir;
                    //            fileSystemWatcher1.NotifyFilter = NotifyFilters.LastWrite;
                    fileSystemWatcher1.NotifyFilter = NotifyFilters.LastAccess | NotifyFilters.LastWrite | NotifyFilters.FileName | NotifyFilters.DirectoryName;

                    fileSystemWatcher2.Path = settings.RefDir;
                    //            fileSystemWatcher2.NotifyFilter = NotifyFilters.LastWrite;
                    fileSystemWatcher2.NotifyFilter = NotifyFilters.LastAccess | NotifyFilters.LastWrite | NotifyFilters.FileName | NotifyFilters.DirectoryName;
                }
                catch (System.ArgumentException)
                {
                }
                catch (System.IO.FileNotFoundException)
                {
                    // no need to handle this
                }
            }
        }

        // from http://stackoverflow.com/questions/1406808/wait-for-file-to-be-freed-by-process/1406853#1406853
        public static bool IsFileReady(String sFilename)
        {
            // If the file can be opened for exclusive access it means that the file
            // is no longer locked by another process.
            try
            {
                using (FileStream inputStream = File.Open(sFilename, FileMode.Open, FileAccess.Read, FileShare.None))
                {
                    if (inputStream.Length > 0)
                    {
                        return true;
                    }
                    else
                    {
                        return false;
                    }

                }
            }
            catch (Exception)
            {
                return false;
            }
        }

        private void fileSystemWatcher1_Changed(object sender, FileSystemEventArgs e)
        {
            // Note: can hang, not sure why

            // wait for file to be finished (see bottom: http://stackoverflow.com/questions/15017506/using-filesystemwatcher-to-monitor-a-directory)
            while (!IsFileReady(e.FullPath))
            {
            }

            {
                PopulateList();
                SetupFileSystemWatcher();
                // todo: Update ListView?
            }
        }

        private void fileSystemWatcher1_Renamed(object sender, RenamedEventArgs e)
        {
            FileSystemEventArgs args = new FileSystemEventArgs(e.ChangeType, e.FullPath, e.Name);

            fileSystemWatcher1_Changed(sender, args);
        }

        private void pictureBoxTest_MouseMove(object sender, MouseEventArgs e)
        {
            bool bWasDrag = MouseStartTrackOffsetX != -1;
            bool bDrag = (e.Button & MouseButtons.Left) != 0;
            float Scale = ComputeScale();

            bool bUpdate = false;

            // usually 120, 0 or -120
            int RelWheel = e.Delta;

            if (RelWheel != 0)
            {
                ImageOffsetX += e.X / Scale;
                ImageOffsetY += e.Y / Scale;

                ImageZoom += RelWheel;
                Scale = ComputeScale();

                ImageOffsetX -= e.X / Scale;
                ImageOffsetY -= e.Y / Scale;

                if (ImageZoom <= 0)
                {
                    ImageZoom = 0;
                    ImageOffsetX = 0;
                    ImageOffsetY = 0;
                }
                bUpdate = true;
            }

            if (bDrag && bWasDrag && ImageZoom > 0)
            {
                int RelX = e.X - MouseStartTrackOffsetX;
                int RelY = e.Y - MouseStartTrackOffsetY;

                // drag

                ImageOffsetX -= RelX / Scale;
                ImageOffsetY -= RelY / Scale;

                // clamp to  border

                ImageOffsetX = Math.Max(0, ImageOffsetX);
                ImageOffsetY = Math.Max(0, ImageOffsetY);

                System.Drawing.Size imageSize = intermediate.GetImagesSize();

                ImageOffsetX = Math.Min(imageSize.Width - pictureBoxTest.Width / Scale, ImageOffsetX);
                ImageOffsetY = Math.Min(imageSize.Height - pictureBoxTest.Height / Scale, ImageOffsetY);

                bUpdate = true;
            }
            if (bDrag)
            {
                MouseStartTrackOffsetX = e.X;
                MouseStartTrackOffsetY = e.Y;
            }
            else
            {
                // disable drag
                MouseStartTrackOffsetX = -1;
                MouseStartTrackOffsetY = 0;
            }

            if (bUpdate)
            {
                pictureBoxTest.Invalidate();
                pictureBoxDiff.Invalidate();
                pictureBoxRef.Invalidate();
                pictureBoxTest.Update();
                pictureBoxDiff.Update();
                pictureBoxRef.Update();
            }
        }

        private void newToolStripMenuNew_Click(object sender, EventArgs e)
        {
            DialogResult d = MessageBox.Show("Are you sure you want to create a new document?\nYour changes will not be saved.", "Question", MessageBoxButtons.YesNo, MessageBoxIcon.Question);

            if (d == DialogResult.Yes)
            {
                ImageValidatorSettings settings = new ImageValidatorSettings();

                SetSettings(settings);
                PopulateList();
            }
        }

        private bool SaveSettings(string FileName)
        {
            SetFileName(saveFileDialog1.FileName);

            ImageValidatorSettings settings = GetSettings();

            XmlSerializer serializer = new XmlSerializer(settings.GetType());

            try
            {
                using (StreamWriter file = new StreamWriter(FileName))
                {
                    serializer.Serialize(file, settings);
                }
            }
            catch (Exception)
            {
                return false;
            }
            return true;
        }

        // settings file name
        private string GetFileName()
        {
            return openFileDialog1.FileName;
        }

        // settings file name
        private void SetFileName(string FileName)
        {
            // set window title
            Text = Path.GetFileName(FileName) + " - ImageValidator " + ImageValidatorSettings.GetVersionString();
            openFileDialog1.FileName = FileName;
        }
        
        private bool LoadSettings(string FileName)
        {
            SetFileName(FileName);

            ImageValidatorSettings settings = new ImageValidatorSettings();

            XmlSerializer serializer = new XmlSerializer(settings.GetType());

            try
            {
                using (StreamReader file = new StreamReader(FileName))
                {
                    settings = serializer.Deserialize(file) as ImageValidatorSettings;
                    SetSettings(settings);
                }

                PopulateList();
                SetupFileSystemWatcher();
            }
            catch (Exception)
            {
                // update UI
                SetSettings(settings);
                return false;
            }

            // update UI
            SetSettings(settings);

            return true;
        }

        private void saveToolStripMenuSave_Click(object sender, EventArgs e)
        {
            if (!SaveSettings(saveFileDialog1.FileName))
            {
                // show SaveAs dialog if needed
                saveAsToolStripMenuSaveAs_Click(sender, e);
            }
        }

        private void saveAsToolStripMenuSaveAs_Click(object sender, EventArgs e)
        {
            if (saveFileDialog1.ShowDialog() == DialogResult.OK)
            {
                SaveSettings(saveFileDialog1.FileName);
            }
        }

        private void quitToolStripMenuQuit_Click(object sender, EventArgs e)
        {
            DialogResult d = MessageBox.Show("Are you sure you want to quit?\nYour changes will not be saved.", "Question", MessageBoxButtons.YesNo, MessageBoxIcon.Question);

            if (d == DialogResult.Yes)
            {
                Application.Exit();
            }
        }

        private void openToolStripMenuItem_Click(object sender, EventArgs e)
        {
            if (openFileDialog1.ShowDialog() == DialogResult.OK)
            {
                LoadSettings(GetFileName());
            }
        }

        private void howToUseWithUnrealEngineToolStripMenuItem_Click(object sender, EventArgs e)
        {
            // todo:
            System.Diagnostics.Process.Start("http://www.google.com");
        }

      /*  // http://stackoverflow.com/questions/19134062/encode-a-filestream-to-base64-with-c-sharp
        public string ConvertToBase64(Stream stream)
        {
            byte[] inArray = new byte[(int)stream.Length];
            stream.Read(inArray, 0, (int)stream.Length);
//            return Convert.ToBase64String(inArray);

            byte[] byt = System.Text.Encoding.UTF8.GetBytes(inArray);

            // convert the byte array to a Base64 string

            return Convert.ToBase64String(byt);
        }
*/
        private void WaitUntilTestIsDone(ref ImageValidatorData data, ref ImageValidatorSettings settings)
        {
            // todo: progress bar, cancel button
            while (data.ProcessOneElement(settings))
            {
                UpdateListView();
            }
        }

        private void GenerateHTMLReport(bool bThumbnails)
        {
//            string f = Clipboard.GetText();
//            HtmlFragment.CopyToClipboard("<b>Hello!</b>");
//            HtmlFragment.CopyToClipboard("<table style=\"width:100%\"><tr><th>Firstname</th><th>Lastname</th><th>Points</th></tr><tr><td>Jill</td><td>Smith</td><td>50</td></tr><tr><td>Eve</td><td>Jackson</td><td>94</td></tr><tr><td>John</td><td>Doe</td><td>80</td></tr></table>");

//            return;

         //   HtmlFragment.CopyToClipboard("<img width=240 height=160 src=\"data:image/gif;base64,R0lGODlhEAAQAMQAAORHHOVSKudfOulrSOp3WOyDZu6QdvCchPGolfO0o/XBs/fNwfjZ0frl3/zy7////wAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAACH5BAkAABAALAAAAAAQABAAAAVVICSOZGlCQAosJ6mu7fiyZeKqNKToQGDsM8hBADgUXoGAiqhSvp5QAnQKGIgUhwFUYLCVDFCrKUE1lBavAViFIDlTImbKC5Gm2hB0SlBCBMQiB0UjIQA7\">");

            if (saveFileDialogExport.ShowDialog() == DialogResult.OK)
            {
                ReportGenerator reportGenerator = new ReportGenerator();

                ImageValidatorSettings settings = GetSettings();

                string SettingsFileName = GetFileName();

                WaitUntilTestIsDone(ref data, ref settings);
                reportGenerator.ExportHTML(SettingsFileName, ref data, ref settings, saveFileDialogExport.FileName, false, bThumbnails);   
            }

/*
//            var image = resizeImage(in
            ImageValidatorSettings settings = GetSettings();termediate.imageRef, 16 * 5, 9 * 5);
            var image = resizeImage(intermediate.imageRef, 16 * 5, 9 * 5);

            MemoryStream stream = new MemoryStream();

//            image.Save("d:\\temp\\da.png", ImageFormat.Png);

            image.Save(stream, ImageFormat.Png);

//            Convert.ToBase64CharArray(inArray, 0, inArray.Length, outArray, 0);

            string Ret = "";
            for (int f = 0; f < 20; ++f)
            {
                Ret += "<img width=\"" + image.Width + "\" height=\"" + image.Height + "\" src=\"data:image/png;base64,";
    //            string Ret = "<img width=240 height=160 src=\"data:image/gif;base64,";

                Ret += Convert.ToBase64String(stream.ToArray());

    //            Ret += ConvertToBase64(stream);
    //            Ret += "R0lGODlhEAAQAMQAAORHHOVSKudfOulrSOp3WOyDZu6QdvCchPGolfO0o/XBs/fNwfjZ0frl3/zy7////wAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAAACH5BAkAABAALAAAAAAQABAAAAVVICSOZGlCQAosJ6mu7fiyZeKqNKToQGDsM8hBADgUXoGAiqhSvp5QAnQKGIgUhwFUYLCVDFCrKUE1lBavAViFIDlTImbKC5Gm2hB0SlBCBMQiB0UjIQA7";

                Ret += "\">\n";
            }
            HtmlFragment.CopyToClipboard(Ret);
 */

/*
                        var bytes = File.ReadAllBytes("C:\\somepath\\picture.png");
                        var b64String = Convert.ToBase64String(bytes);
                        var dataUrl = "data:image/png;base64," + b64String;
            */

//            image. ToBase64String();
        }

        private void MenuItemGenerateHTMLReport_Click(object sender, EventArgs e)
        {
            GenerateHTMLReport(false);
        }

        private void MenuItemGenerateHTMLReportWithT_Click(object sender, EventArgs e)
        {
            GenerateHTMLReport(true);
        }

        private void Refresh_Click(object sender, EventArgs e)
        {
            PopulateList();
        }
    }
}
