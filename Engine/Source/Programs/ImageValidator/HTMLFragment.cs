// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace ImageValidator
{
// Sample class for Copying and Pasting HTML fragments to and from the clipboard.

//

// Mike Stall. http://blogs.msdn.com/jmstall

//

using System;

using System.Diagnostics;

using System.Windows.Forms;

using System.Text.RegularExpressions;


using System.Collections.Generic;

using System.Text;

using System.IO;


/// <summary>

/// Helper class to decode HTML from the clipboard.

/// See http://blogs.msdn.com/jmstall/archive/2007/01/21/html-clipboard.aspx for details.

/// </summary>

class HtmlFragment

{
    #region Write to Clipboard

    // Helper to convert an integer into an 8 digit string.
    // String must be 8 characters, because it will be used to replace an 8 character string within a larger string.
    static string To8DigitString(int x)
    {
        return String.Format("{0,8}", x);
    }


    /// <summary>
    /// Clears clipboard and copy a HTML fragment to the clipboard. This generates the header.
    /// </summary>
    /// <param name="htmlFragment">A html fragment.</param>
    /// <example>
    ///    HtmlFragment.CopyToClipboard("<b>Hello!</b>");
    /// </example>
    public static void CopyToClipboard(string htmlFragment)
    {
        CopyToClipboard(htmlFragment, null, null);
    }


    /// <summary>
    /// Clears clipboard and copy a HTML fragment to the clipboard, providing additional meta-information.
    /// </summary>
    /// <param name="htmlFragment">a html fragment</param>
    /// <param name="title">optional title of the HTML document (can be null)</param>
    /// <param name="sourceUrl">optional Source URL of the HTML document, for resolving relative links (can be null)</param>
    public static void CopyToClipboard(string htmlFragment, string title, Uri sourceUrl)
    {
        if (title == null) title = "From Clipboard"; 

        System.Text.StringBuilder sb = new System.Text.StringBuilder();

        // Builds the CF_HTML header. See format specification here:
        // http://msdn.microsoft.com/library/default.asp?url=/workshop/networking/clipboard/htmlclipboard.asp
        // The string contains index references to other spots in the string, so we need placeholders so we can compute the offsets.
        // The <<<<<<<_ strings are just placeholders. We’ll backpatch them actual values afterwards.
        // The string layout (<<<) also ensures that it can’t appear in the body of the html because the <
        // character must be escaped.

        string header = @"Version:1.0
StartHTML:<<<<<<<1
EndHTML:<<<<<<<2
StartFragment:<<<<<<<3
EndFragment:<<<<<<<4
StartSelection:<<<<<<<3
EndSelection:<<<<<<<3
";

        string pre = @"<!DOCTYPE HTML PUBLIC ""-//W3C//DTD HTML 4.0 Transitional//EN"">
<HTML><HEAD><TITLE>" + title + @"</TITLE></HEAD><BODY>";

        string post = @"</BODY></HTML>";

        sb.Append(header);

        if (sourceUrl != null)
        {
            sb.AppendFormat("SourceURL:{0}", sourceUrl);
        }

        int startHTML = sb.Length;

        sb.Append(pre);
        int fragmentStart = sb.Length;

        sb.Append(htmlFragment);
        int fragmentEnd = sb.Length;

        sb.Append(post);
        int endHTML = sb.Length;
        
        // Backpatch offsets
        sb.Replace("<<<<<<<1", To8DigitString(startHTML));
        sb.Replace("<<<<<<<2", To8DigitString(endHTML));
        sb.Replace("<<<<<<<3", To8DigitString(fragmentStart));
        sb.Replace("<<<<<<<4", To8DigitString(fragmentEnd));

        // Finally copy to clipboard.

        string data = sb.ToString();
        Clipboard.Clear();
        Clipboard.SetText(data, TextDataFormat.Html);
    }

    #endregion // Write to Clipboard

    } // end of class
}
