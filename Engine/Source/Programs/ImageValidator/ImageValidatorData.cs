// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Drawing;           // Color
using System.IO;                // Directory.
using System.Diagnostics;       // Debug.
using System.Drawing.Imaging;   // ImageData

namespace ImageValidator
{
    // per folder, later stored in xml file
    public class ImageValidatorSettings
    {
        public string TestDir;
        public string RefDir;
        public uint Threshold;
        public uint PixelCountToFail;

        public ImageValidatorSettings()
        {
            Threshold = 16;
            PixelCountToFail = 32;
        }

        public static string GetVersionString()
        {
            return "V1.0";
        }
    };

    public struct ImageValidatorIntermediate
    {
        public struct PixelElement
        {
            // blue, green, red, alpha
            public byte b, g, r, a;

            public void SetErrorColor(float squaredError)
            {
                // opaque
                a = 0xff;

                r = 0xff;
                g = 0;
                b = 0;

                g = (byte)(Math.Min(0xff, squaredError / 8));
                b = (byte)(Math.Min(0xff, squaredError / 64));
            }

            /*
            public void SetErrorColor(float squaredError)
            {
                a = 0xff;

                if (squaredError > 0.0f)
                {
                    // no error is black, minor error is a noticeable color
                    squaredError += 800;
                }
                squaredError /= 8;

                r = (byte)(Math.Min(0xff, squaredError));
                g = (byte)(Math.Min(0xff, squaredError / 8));
                b = (byte)(Math.Min(0xff, squaredError / 64));
            }
            */

            public static float ComputeSquaredError(PixelElement Test, PixelElement Ref)
            {
                float R = (float)Test.r - (float)Ref.r;
                float G = (float)Test.g - (float)Ref.g;
                float B = (float)Test.b - (float)Ref.b;

                return R * R + G * G + B * B;
            }
            public static uint ComputeAbsDiff(PixelElement Test, PixelElement Ref)
            {
                int R = Math.Abs((int)Test.r - (int)Ref.r);
                int G = Math.Abs((int)Test.g - (int)Ref.g);
                int B = Math.Abs((int)Test.b - (int)Ref.b);

                return (uint)Math.Max(R, Math.Max(G, B));
            }
        };

        public Bitmap imageTest;
        public Bitmap imageDiff;
        public Bitmap imageRef;

        private Bitmap LoadBitmap(string filename)
        {
            Image tmpImage;
            Bitmap Ret = null;

            try
            {
                // http://stackoverflow.com/questions/788335/why-does-image-fromfile-keep-a-file-handle-open-sometimes
                // load without keeping the file locked

                using (var fs = new FileStream(filename, FileMode.Open, FileAccess.Read))
                {
                    tmpImage = Image.FromStream(fs);
                    Ret = new Bitmap(tmpImage);
                    tmpImage.Dispose();
                }
            }
            catch (System.IO.DirectoryNotFoundException)
            {
                // no need to handle this
            }
            catch (System.IO.FileNotFoundException)
            {
                // no need to handle this
            }

            return Ret;
        }

        public void Process(ImageValidatorSettings settings, ImageValidatorData.ImageEntry imageEntry)
        {
            if (imageTest != null) { imageTest.Dispose(); imageTest = null; }
            if (imageRef != null) { imageRef.Dispose(); imageRef = null; }

            imageTest = LoadBitmap(settings.TestDir + imageEntry.Name);
            imageRef = LoadBitmap(settings.RefDir + imageEntry.Name);

            imageEntry.testResult = ComputeDiff(settings.Threshold);
        }

        public static System.Drawing.Size GetImageSize(Bitmap bitmap)
        {
            GraphicsUnit Unit = GraphicsUnit.Pixel;

            System.Drawing.Size Ret = new System.Drawing.Size(1, 1);

            if (bitmap != null)
            {
                RectangleF bounds = bitmap.GetBounds(ref Unit);

                Ret.Width = (int)bounds.Width;
                Ret.Height = (int)bounds.Height;
            }

            return Ret;
        }

        public System.Drawing.Size GetImagesSize()
        {
            System.Drawing.Size sizeTest = GetImageSize(imageTest);
            System.Drawing.Size sizeRef = GetImageSize(imageRef);

            return new System.Drawing.Size(
                Math.Max(sizeTest.Width, sizeRef.Width),
                Math.Max(sizeTest.Height, sizeRef.Height)
                );
        }

        public static Bitmap resizeImage(Image image)
        {
//            uint MaxSize = 64;
            uint MaxSize = 128;

            float Scale = MaxSize / (float)Math.Max(image.Width, image.Height);

            int new_width = Math.Max(1, (int)(image.Width * Scale));
            int new_height = Math.Max(1, (int)(image.Height * Scale));

            // see http://base64image.org/

            Bitmap new_image = new Bitmap(new_width, new_height);
            Graphics g = Graphics.FromImage((Image)new_image);
            //g.InterpolationMode = System.Drawing.Drawing2D.InterpolationMode.High;    // creates alpha channel from border pixels - doesn't look good for the Report 
            g.InterpolationMode = System.Drawing.Drawing2D.InterpolationMode.Low;
            g.DrawImage(image, 0, 0, new_width, new_height);

            return new_image;
        }

        // compute imageDiff from imageTest and imageRef
        public TestResult ComputeDiff(uint Threshold)
        {
            if (imageDiff != null)
            {
                imageDiff.Dispose();
                imageDiff = null;
            }

            TestResult Ret = new TestResult();
            if (imageTest == null)
            {
                Ret.ErrorText = "missing Test";
                return Ret;
            }
            if (imageRef == null)
            {
                Ret.ErrorText = "missing Ref";
                return Ret;
            }

            System.Drawing.Size sizeTest = GetImageSize(imageTest);
            System.Drawing.Size sizeRef = GetImageSize(imageRef);

            if (sizeTest != sizeRef)
            {
                Ret.ErrorText = "Size " +
                    sizeTest.Width.ToString() + "x" + sizeTest.Height.ToString() +
                    " != " +
                    sizeRef.Width.ToString() + "x" + sizeRef.Height.ToString();
                return Ret;
            }

            System.Drawing.Size size = GetImagesSize();

            // todo: exit if Size or format is not the same
            Debug.Assert(imageTest.PixelFormat == PixelFormat.Format32bppArgb);

            imageDiff = new Bitmap(size.Width, size.Height);

            uint CountedErrorPixels = 0;

            BitmapData dataTest = imageTest.LockBits(new Rectangle(0, 0, size.Width, size.Height), ImageLockMode.ReadOnly, imageTest.PixelFormat);
            BitmapData dataDiff = imageDiff.LockBits(new Rectangle(0, 0, size.Width, size.Height), ImageLockMode.WriteOnly, imageTest.PixelFormat);
            BitmapData dataRef = imageRef.LockBits(new Rectangle(0, 0, size.Width, size.Height), ImageLockMode.ReadOnly, imageTest.PixelFormat);
            unsafe
            {
                for (int x = 0; x < size.Width; ++x)
                {
                    for (int y = 0; y < size.Height; ++y)
                    {
                        PixelElement* valueTest = (PixelElement*)((byte*)dataTest.Scan0.ToPointer() + dataTest.Stride * y + x * sizeof(PixelElement));
                        PixelElement* valueDiff = (PixelElement*)((byte*)dataDiff.Scan0.ToPointer() + dataDiff.Stride * y + x * sizeof(PixelElement));
                        PixelElement* valueRef = (PixelElement*)((byte*)dataRef.Scan0.ToPointer() + dataRef.Stride * y + x * sizeof(PixelElement));

                        //                            float Diff = PixelElement.ComputeSquaredError(*valueDiff, *valueRef);
                        uint localError = PixelElement.ComputeAbsDiff(*valueTest, *valueRef);

                        // all pixels opaque
                        valueDiff->a = 0xff;

                        if (localError >= Threshold)
                        {
                            ++CountedErrorPixels;
                            //                            valueDiff->SetErrorColor(PixelElement.ComputeSquaredError(*valueTest, *valueRef));
                            valueDiff->SetErrorColor(localError);
                            //                            *valueDiff = *valueRef;
                        }
                    }
                }
            }
            imageRef.UnlockBits(dataRef);
            imageDiff.UnlockBits(dataDiff);
            imageTest.UnlockBits(dataTest);

            Ret.ErrorPixels = CountedErrorPixels;

            // update Thumbnails
            {
                Ret.ThumbnailTest = resizeImage(imageTest);
                Ret.ThumbnailDiff = resizeImage(imageDiff);
                Ret.ThumbnailRef = resizeImage(imageRef);
            }



            return Ret;
        }
    };
    
    public class TestResult
    {
        public static Color GetColor(bool bPassed)
        {
            return bPassed ? Color.FromArgb(0x88ff88) : Color.FromArgb(0xff8888);
        }

        public Color GetColor(ref ImageValidatorSettings settings)
        {
            return GetColor(IsPassed(ref settings));
        }

        public bool IsPassed(ref ImageValidatorSettings settings)
        {
            return ErrorText == null && ErrorPixels < settings.PixelCountToFail;
        }

        public string GetString(ref ImageValidatorSettings settings)
        {
            if (ErrorText != null)
            {
                return ErrorText;
            }
            else
            {
                string Ret = IsPassed(ref settings) ? "Passed" : "Failed";

                Ret += " (" + ErrorPixels.ToString() + ")";

                return Ret;
            }
        }

        //
        public string ErrorText;
        //  only used if ErrorText is null
        public uint ErrorPixels;

        public Bitmap ThumbnailTest;
        public Bitmap ThumbnailDiff;
        public Bitmap ThumbnailRef;
    };

    public struct ImageValidatorData
    {
        public struct ImageEntryColumnData
        {
            public string Platform;
            public string Map;
            public string Time;
            public string Actor;

            public ImageEntryColumnData(string Name)
            {
                Platform = Path.GetDirectoryName(Name);
                if (Platform.StartsWith(@"\"))
                {
                    Platform = Platform.Substring(1);
                }

                string FileNameWithoutExtension = Path.GetFileNameWithoutExtension(Name);
                string[] keyValuePairs = FileNameWithoutExtension.Split(' ');

                Map = "";
                Time = "";
                Actor = "";

                foreach (string keyvalue in keyValuePairs)
                {
                    string trimKeyValue = keyvalue;

                    if (trimKeyValue.EndsWith(")"))
                    {
                        trimKeyValue = trimKeyValue.Substring(0, trimKeyValue.Length - 1);
                    }

                    if (trimKeyValue.StartsWith("Map("))
                    {
                        Map = trimKeyValue.Substring(4);
                    }
                    if (trimKeyValue.StartsWith("Time("))
                    {
                        Time = trimKeyValue.Substring(5);
                    }
                    if (trimKeyValue.StartsWith("Actor("))
                    {
                        Actor = trimKeyValue.Substring(6);
                    }
                }

                // clernup some legacy nameing convention
                if (Time.EndsWith("s"))
                {
                    Time = Time.Substring(0, Time.Length - 1);
                }
            }
        }

        public class ImageEntry : IComparable 
        {
            public ImageEntry()
            {
                bRefExists = false;
                bTestExists = false;
            }

            // key, without front path
            public string Name;
            //
            public bool bRefExists;
            //
            public bool bTestExists;
            // can be null
            public TestResult testResult;

            public int CompareTo(object _rhs)
            {
                ImageEntry rhs = _rhs as ImageEntry;

                if (rhs == null)
                    return 1;

                ImageValidatorData.ImageEntryColumnData columnThis = new ImageValidatorData.ImageEntryColumnData(Name);
                ImageValidatorData.ImageEntryColumnData columnRhs = new ImageValidatorData.ImageEntryColumnData(rhs.Name);

                try
                {
                    float timeThis = float.Parse(columnThis.Time);
                    float timeRhs = float.Parse(columnRhs.Time);

                    int x = timeThis.CompareTo(timeRhs);

                    if(x != 0)
                        return x;
                }
                catch (Exception)
                {
                    // if time is not part of the name we cannot sort by it
                }

                return Name.CompareTo(rhs.Name);
            }
        };


        // --------------------------------------------------------------------

        public List<ImageEntry> imageEntries;

        private void PopulatePartList(bool bRef, string path)
        {
            try
            {
                string[] files = Directory.GetFiles(path, "*.png", SearchOption.AllDirectories);
                int pathlen = path.Length;

                foreach (string it in files)
                {
                    string substr = it.Substring(pathlen);

                    ImageEntry thisEntry = null;

                    foreach (ImageEntry entry in imageEntries)
                    {
                        if (entry.Name == substr)
                        {
                            thisEntry = entry;
                            break;
                        }
                    }

                    if (thisEntry == null)
                    {
                        thisEntry = new ImageEntry();
                        thisEntry.Name = substr;
                        imageEntries.Add(thisEntry);
                    }

                    if (bRef)
                    {
                        thisEntry.bRefExists = true;
                    }
                    else
                    {
                        thisEntry.bTestExists = true;
                    }
                }
            }
            catch (System.ArgumentException)
            {
            }
            catch (System.IO.DirectoryNotFoundException)
            {
            }
            catch (System.IO.FileNotFoundException)
            {
            }
        }

        // @retur if a element was processed
        public bool ProcessOneElement(ImageValidatorSettings settings)
        {
            foreach (ImageEntry entry in imageEntries)
            {
                if (entry.testResult == null)
                {
                    ImageValidatorIntermediate intermediate = new ImageValidatorIntermediate();
                    intermediate.Process(settings, entry);
                    return true;
                }
            }

            return false;
        }

        public void PopulateList(string TestDir, string RefDir)
        {
            imageEntries = new List<ImageEntry>();
            PopulatePartList(false, TestDir);
            PopulatePartList(true, RefDir);
            imageEntries.Sort();
        }
    }
}
