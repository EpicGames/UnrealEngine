// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Windows.Forms;     // SortOrder

namespace ImageValidator
{
    // Compares two ListView items based on a selected column.
    // from http://csharphelper.com/blog/2014/09/sort-a-listview-using-the-column-you-click-in-c
    public class ListViewComparer : System.Collections.IComparer
    {
        private int ColumnNumber;
        private SortOrder SortOrder;

        public ListViewComparer(int column_number,
            SortOrder sort_order)
        {
            ColumnNumber = column_number;
            SortOrder = sort_order;
        }

        // Compare two ListViewItems.
        public int Compare(object object_x, object object_y)
        {
            // Get the objects as ListViewItems.
            ListViewItem item_x = object_x as ListViewItem;
            ListViewItem item_y = object_y as ListViewItem;

            // Get the corresponding sub-item values.
            string string_x;
            if (item_x.SubItems.Count <= ColumnNumber)
            {
                string_x = "";
            }
            else
            {
                string_x = item_x.SubItems[ColumnNumber].Text;
            }

            string string_y;
            if (item_y.SubItems.Count <= ColumnNumber)
            {
                string_y = "";
            }
            else
            {
                string_y = item_y.SubItems[ColumnNumber].Text;
            }

            // Compare them.
            int result;
            double double_x, double_y;
            if (double.TryParse(string_x, out double_x) &&
                double.TryParse(string_y, out double_y))
            {
                // Treat as a number.
                result = double_x.CompareTo(double_y);
            }
            else
            {
                DateTime date_x, date_y;
                if (DateTime.TryParse(string_x, out date_x) &&
                    DateTime.TryParse(string_y, out date_y))
                {
                    // Treat as a date.
                    result = date_x.CompareTo(date_y);
                }
                else
                {
                    // Treat as a string.
                    result = string_x.CompareTo(string_y);
                }
            }

            // Return the correct result depending on whether
            // we're sorting ascending or descending.
            if (SortOrder == SortOrder.Ascending)
            {
                return result;
            }
            else
            {
                return -result;
            }
        }
    }
}
