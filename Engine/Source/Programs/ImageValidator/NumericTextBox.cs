// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Windows.Forms;
using System.Text;
using System.Globalization;
using System.Threading.Tasks;

// see https://msdn.microsoft.com/en-us/library/ms229644(v=vs.90).aspx

namespace ImageValidator
{
    class NumericTextBox : TextBox
    {
        bool allowSpace = false;

        // Restricts the entry of characters to digits (including hex), the negative sign, 
        // the decimal point, and editing keystrokes (backspace). 
        protected override void OnKeyPress(KeyPressEventArgs e)
        {
            base.OnKeyPress(e);

            NumberFormatInfo numberFormatInfo = System.Globalization.CultureInfo.CurrentCulture.NumberFormat;
            string decimalSeparator = numberFormatInfo.NumberDecimalSeparator;
            string groupSeparator = numberFormatInfo.NumberGroupSeparator;
            string negativeSign = numberFormatInfo.NegativeSign;

            // Workaround for groupSeparator equal to non-breaking space 
            if (groupSeparator == ((char)160).ToString())
            {
                groupSeparator = " ";
            }

            string keyInput = e.KeyChar.ToString();

            if (Char.IsDigit(e.KeyChar))
            {
                // Digits are OK
            }
            else if (keyInput.Equals(decimalSeparator) || keyInput.Equals(groupSeparator) ||
             keyInput.Equals(negativeSign))
            {
                // Decimal separator is OK
            }
            else if (e.KeyChar == '\b')
            {
                // Backspace key is OK
            }
            //    else if ((ModifierKeys & (Keys.Control | Keys.Alt)) != 0) 
            //    { 
            //     // Let the edit control handle control and alt key combinations 
            //    } 
            else if (this.allowSpace && e.KeyChar == ' ')
            {

            }
            else
            {
                // Consume this invalid key and beep
                e.Handled = true;
                //    MessageBeep();
            }
        }

        public int IntValue
        {
            get
            {
                return Int32.Parse(this.Text);
            }
        }

        public decimal DecimalValue
        {
            get
            {
                return Decimal.Parse(this.Text);
            }
        }

        public bool AllowSpace
        {
            set
            {
                this.allowSpace = value;
            }

            get
            {
                return this.allowSpace;
            }
        }
    }
}
