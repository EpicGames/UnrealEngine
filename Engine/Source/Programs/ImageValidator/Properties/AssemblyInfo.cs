// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using System.Runtime.CompilerServices;
using System.Runtime.InteropServices;

// General Information about an assembly is controlled through the following 
// set of attributes. Change these attribute values to modify the information
// associated with an assembly.
[assembly: AssemblyTitle("ImageValidator")]
[assembly: AssemblyDescription("")]
[assembly: AssemblyConfiguration("")]
[assembly: AssemblyCompany("Epic Games")]
[assembly: AssemblyProduct("ImageValidator")]
[assembly: AssemblyCopyright("Copyright Epic Games, Inc.")]
[assembly: AssemblyTrademark("")]
[assembly: AssemblyCulture("")]

// Setting ComVisible to false makes the types in this assembly not visible 
// to COM components.  If you need to access a type in this assembly from 
// COM, set the ComVisible attribute to true on that type.
[assembly: ComVisible(false)]

// The following GUID is for the ID of the typelib if this project is exposed to COM
[assembly: Guid("0803a40e-74ea-4d28-ac80-acbf3197a06c")]

// Version information for an assembly consists of the following four values:
//
//      Major Version
//      Minor Version 
//      Build Number
//      Revision
//
// You can specify all the values or you can default the Build and Revision Numbers 
// by using the '*' as shown below:
// [assembly: AssemblyVersion("1.0.*")]
[assembly: AssemblyVersion("1.0.0.0")]
[assembly: AssemblyFileVersion("1.0.0.0")]
