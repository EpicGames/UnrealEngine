

todo:
* store last folder[s] in registry
* xml in ref folder 
* documentation (hookup help and about)
* summary line in UI
* ref folder is read only
* Generate Report without Thumbnails to Clipboard
* remove leading \ from folder name
* Report: Sort bad ones up? Summary
* FileSystemWatcher should update only the file that has changed (when running the test and watching it will be more efficient)
* command line args, see https://msdn.microsoft.com/en-us/library/system.io.filesystemwatcher(v=vs.110).aspx
* better error handling e.g. if cannot load png
* send out email (requires IT or running on Server where we allow SMTP)
* comparison can happen in a thread
* move out of DoNotDistribute
* Diff downsample should be conservative, not point sampling (any difference in a pixel should be always highlighted - no matte what scale)

later todo:
* test zoom and pan with wacom
* nicer folder open dialog

---------------------------------------------------



features:
* C# application
* allows comparison of png color (no alpha) images in two specified folders (all 3 shown at the same time)
* comparison is based on pixel difference (current metric: max(abs(delta_r), abs(delta_g), abs(delta_b)) )
* scalable dialog
* click on column allows sorting
* changes to the folder automatically triggers folder processing again (can be improved to nly do the chanaged files)
* ...

future:
* command line processing to run with automated build server
* multithreading for better user interacation (not an issue with reasonable image sizes)
* Save folder settings to a file, accept reference button (interop with Perforce)
* Zoom in, Pan
* Other comparison metrics / visualization
* Summary control / progress bar
* Nicer Application Icon
