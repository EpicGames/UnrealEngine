// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.IO;                // Path
using System.Drawing;           // Color
using System.Drawing.Imaging;   // ImageData
using System.Diagnostics;       // Debug.
using System.Windows.Forms;     // MessageBox

namespace ImageValidator
{
    class ReportGenerator
    {
        private struct ValidationSummary
        {
            public uint Failed;
            public uint Succeeded;

            public ValidationSummary(ref ImageValidatorData data, ref ImageValidatorSettings settings)
            {
                Failed = 0;
                Succeeded = 0;

                foreach (ImageValidatorData.ImageEntry entry in data.imageEntries)
                {
                    TestResult test = entry.testResult;

                    if (test.IsPassed(ref settings))
                    {
                        ++Succeeded;
                    }
                    else
                    {
                        ++Failed;
                    }
                }
            }
        };

        public void ExportHTML(string SettingsFileName, ref ImageValidatorData data, ref ImageValidatorSettings settings, string FileName, bool bSilent, bool bThumbnails)
        {
            ValidationSummary validationSummary = new ValidationSummary(ref data, ref settings);

            string Folder = Path.GetDirectoryName(FileName) + "\\" + Path.GetFileNameWithoutExtension(FileName) + "_Thumbnails";

            if (bThumbnails)
            {
                Directory.CreateDirectory(Folder);
            }

            // Note C# string literals can span multiple lines, " need to be converted to ""
            string template = @"
<!DOCTYPE HTML PUBLIC \""-//W3C//DTD HTML 4.01 Transitional//EN\"" \""http://www.w3.org/TR/html4/loose.dtd\"">
<html>
<head>
<title>ImageValidator Report</title>
</head>
<body>
<h1>ImageValidator Report</h1>

<br>
<h4>ImageValidator Settings ""%SETTINGSFILENAME%"":</h4>
<div style=""margin:10px;margin-left:40px"" >%SETTINGS% </div>
<br>

<h4>Summary:</h4>
<div style=""margin:10px;margin-left:40px""> %SUMMARY% </div>
<br>

<h4>Output:</h4>
<div style=""margin:10px;margin-left:40px""> %OUTPUT% </div>
<br>

<br>
<b>Generated by Application:</b> ImageValidator.exe (part of UnrealEngine by Epic Games)<br>
<b>Application Version:</b> %VERSION%<br>
<b>Generated by User:</b> %USER%<br>
<b>Generated Date:</b> %DATE%<br>

<style type=""text/css"">
//body { background-color:#000000; color:#E0E0E0 }

* {
    font-family: ?Times New Roman?, Times, serif;
 //   font-weight: bold;
    
}

h1 { color:#555555; font-size:32px; border-bottom:solid thin black; }

table, hd, th {
    border:0px; color:#000000; padding:4px; border-spacing:0px;
    border-collapse:collapse;
    }
   
td { padding:2px; padding-right:7px }
td.Output { padding:7px; border:1px solid #cccccc; }

th {
    background-color:#E0E0E0;
    color: #000000;
    text-align: left;
    border: 1px solid #c0c0c0;
}

a.prefix {
    border: 0px solid;
    padding: 3px;
    padding-left: 5px;
    padding-right: 5px;
    font-weight: bold;
    text-decoration:none;
    margin:4px;
    white-space: nowrap;
}

a.prefix:link {
    border:1px solid #7777FF;
    background-color:#ddddff;
    color: #7777FF;
}


a.prefix:hover {
    background-color:#7777FF;
    color: #ddddff;
}

</style>
    
</body>
</html>";

            // we can change this if needed:
            //ImageFormat imageFormat = ImageFormat.Jpeg;   // small file size?
            //ImageFormat imageFormat = ImageFormat.Gif;   // dither artifacts
            ImageFormat imageFormat = ImageFormat.Png;    // good compromise

            // we derive the extension from imageFormat e.g. ".jpg"
            string imageFormatExtension = "";
            {
                if (imageFormat == ImageFormat.Jpeg)
                {
                    imageFormatExtension = ".jpg";
                }
                else if (imageFormat == ImageFormat.Gif)
                {
                    imageFormatExtension = ".gif";
                }
                else if (imageFormat == ImageFormat.Png)
                {
                    imageFormatExtension = ".Png";
                }

                Debug.Assert(imageFormatExtension != "");
            }

            // todo: how to repro, Date, IVxml file attached?, thumbnails

            string Output = "<table class=Output><tr>";

            Output += "<th>Result (pixels failed)</th>";

            if (bThumbnails)
            {
                Output += "<th>Thumbnails (test/diff/ref)</th>";
            }
            Output += "<th>Matinee Location (in sec)</th>";
            //            Output += "<th>Actor</th>";
            //            Output += "<th>Map</th>";
            //            Output += "<th>Platform</th>";
            Output += "<th>Folder Name</th>";
            Output += "<th>File Name</th></tr>\n";
            {
                uint Index = 0;
                foreach (ImageValidatorData.ImageEntry entry in data.imageEntries)
                {
                    TestResult test = entry.testResult;
                    bool bShowDiffAndRefThumbnails = !test.IsPassed(ref settings);

                    // open table row
                    Output += "<tr class=Output>";

                    Color BackColor = Color.White;

                    string ThumbnailTest = Folder + "\\Test" + Index + imageFormatExtension;
                    string ThumbnailDiff = Folder + "\\Diff" + Index + imageFormatExtension;
                    string ThumbnailRef = Folder + "\\Ref" + Index + imageFormatExtension;

                    if (test != null)
                    {
                        BackColor = test.GetColor(ref settings);

                        if (bThumbnails)
                        {
                            test.ThumbnailTest.Save(ThumbnailTest, imageFormat);

                            if (bShowDiffAndRefThumbnails)
                            {
                                test.ThumbnailDiff.Save(ThumbnailDiff, imageFormat);
                                test.ThumbnailRef.Save(ThumbnailRef, imageFormat);
                            }
                        }
                    }

                    ImageValidatorData.ImageEntryColumnData columnData = new ImageValidatorData.ImageEntryColumnData(entry.Name);

                    string ResultString = "?";

                    if (entry.testResult != null)
                    {
                        ResultString = entry.testResult.GetString(ref settings);
                    }

                    Output += "<td class=Output bgcolor=" + ColorTranslator.ToHtml(BackColor) + ">" + ResultString + "</td>";

                    // thumbnails
                    if (bThumbnails)
                    {
                        Output += "<td class=Output>";

                        if (test != null)
                        {
                            Output += "<img src=\"" + ThumbnailTest + "\" width=" + test.ThumbnailTest.Width + " height=" + test.ThumbnailTest.Height + ">";

                            if (bShowDiffAndRefThumbnails)
                            {
                                Output += "<img src=\"" + ThumbnailDiff + "\" width=" + test.ThumbnailDiff.Width + " height=" + test.ThumbnailDiff.Height + ">";
                                Output += "<img src=\"" + ThumbnailRef + "\" width=" + test.ThumbnailRef.Width + " height=" + test.ThumbnailRef.Height + ">";
                            }
                        }

                        Output += "</td>";
                    }

                    Output += "<td class=Output>" + columnData.Time + "</td>";
                    //                    Output += "<td class=Output>" + columnData.Actor + "</td>";
                    //                    Output += "<td class=Output>" + columnData.Map + "</td>";
                    //                    Output += "<td class=Output>" + columnData.Platform + "</td>";
                    Output += "<td class=Output>" + Path.GetDirectoryName(entry.Name) + "</td>";
                    Output += "<td class=Output>" + Path.GetFileName(entry.Name) + "</td>";

                    // close table row
                    Output += "</tr>\n";
                    ++Index;
                }
                Output += "</table>\n";
            }

            string SettingsString = "";
            {
                SettingsString +=
                    "<table>\n" +
                    "<tr><td>Test Directory:</td> <td>" + settings.TestDir + "</td></tr>\n" +
                    "<tr><td>Reference Directory:</td> <td>" + settings.RefDir + "</td></tr>\n" +
                    "<tr><td>Threshold (in 0..255 range):</td> <td>" + settings.Threshold + "</td></tr>\n" +
                    "<tr><td>PixelCountToFail:</td> <td>" + settings.PixelCountToFail + "</td></tr>\n" +
                    "</table>\n";
            }

            string SummaryString = "";
            {
                bool bPassed = validationSummary.Failed == 0;

                string FailedColor = "";
                string SuceededColor = "";

                if (bPassed)
                {
                    SuceededColor = " bgcolor=" + ColorTranslator.ToHtml(TestResult.GetColor(true));
                }
                else
                {
                    FailedColor = " bgcolor=" + ColorTranslator.ToHtml(TestResult.GetColor(false));
                }
                
                SummaryString +=
                    "<table>\n" +
//                    "<tr><td>Files processed:</td> <td>" + (validationSummary.Failed + validationSummary.Succeeded) + "</td></tr>\n" +
                    "<tr><td" + FailedColor + ">Failed:</td> <td" + FailedColor + ">" + validationSummary.Failed + "</td></tr>\n" +
                    "<tr><td" + SuceededColor + ">Succeeded:</td> <td" + SuceededColor + ">" + validationSummary.Succeeded + "</td></tr>\n" +
                    "</table>\n";
            }

            template = template.Replace("%DATE%", DateTime.Now.ToString());
            template = template.Replace("%SETTINGSFILENAME%", SettingsFileName);

            template = template.Replace("%SETTINGS%", SettingsString);
            template = template.Replace("%SUMMARY%", SummaryString);

            template = template.Replace("%OUTPUT%", Output);
            template = template.Replace("%USER%", Environment.UserName);
            template = template.Replace("%VERSION%", ImageValidatorSettings.GetVersionString());


            System.IO.File.WriteAllText(FileName, template);

            if (!bSilent)
            {
                MessageBox.Show("Export to HTML successful", "ImageValidator", MessageBoxButtons.OK);
            }
        }
    }
}
